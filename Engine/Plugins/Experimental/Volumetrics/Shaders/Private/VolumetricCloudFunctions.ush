// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#define UseDetailTexture 0
#define NumShadowSteps 16

const float TanHalfFOV = 1;
const float3 SO = float3(0, 0, PlanetRadius);
const float3 RayOrigin = ResolvedView.WorldCameraOrigin + SO;
const float3 RayDirection = -Parameters.CameraVector;
const float3 LightVector = MaterialExpressionSkyAtmosphereLightDirection(Parameters, 0);

float SD = CalcSceneDepth(ScreenAlignedPosition(GetScreenPosition(Parameters))) / abs(dot(RayDirection, ResolvedView.ViewForward));
float2 ViewRes = View.ViewSizeAndInvSize.xy;

struct CloudScene
{
	//Params from Custom Node Input
	FMaterialPixelParameters Parameters;
	float GroundRadius;
	float Steps;
	float DensityStep;
	float ShadowStep;
	Texture2D CloudMaskTexture;
	SamplerState CloudMaskTextureSampler;
	float CloudMaskScale;
	float CloudMaskBias;
	float2 OffsetXY;
	float CloudLayerAltitude;
	float CloudLayerThickness;
	float CloudUpperFalloff;
	float CloudLowerFalloff;
	Texture3D CloudVolumeTexture;
	SamplerState CloudVolumeTextureSampler;
	float DetailScale_a;
	float DetailAmount_a;
	float DetailMip_a;
	float DetailAltitudeShift_a;
	float DetailScale_b;
	float DetailAmount_b;
	float DetailMip_b;
	float3 SunColor;
	float3 SkyColor;
	float3 GroundColor;
	float InternalScatterBoost;
	float InternalScatterPower;
	float4 ScatterDensityOctaves;
	float4 ScatterIntensityOctaves;
	float3 Panner;
	float CloudDensity;
	float ShadowDensity;
	float AmbientDensity;
	float PowderDensity;
	
	//View Setup
	float3 RO;
	float3 RD;
	float3 LV;
	float SceneDepth;
	float TanHalfFOV;
	float VolumeResolution;
	float2 ViewSize;
	
	//locals
	float3 LightEnergy;
	float Transmittance;
	float accumdist;
	
	float3 SphereIntersection(float inRadius)
	{
		float B = dot(RD, RO);
		float C = dot(RO, RO) - inRadius * inRadius;
	
		float t0 = -B - sqrt(B * B - C);
		float t1 = -B + sqrt(B * B - C);
		
		t0 = max(t0, 0);
		t1 = (t1 > 0) ? min(t1, SceneDepth) : t1;
	
		return float3(t0, t1, max(0, t1 - t0));
	}
	
	float GetMipLevel(float RayDistance, float RepeatSize)
	{
		float TexelsPerPixel = ((RayDistance * TanHalfFOV * 2) / RepeatSize) * (VolumeResolution / max(ViewSize.x, ViewSize.y));
		return max(0, log2(TexelsPerPixel));

	}
	
	float GetNormalizedRingDistance(float3 Pos, float ringnoise)
	{
		float radialdist = length(Pos);
		radialdist -= GroundRadius + CloudLayerAltitude;
		radialdist /= CloudLayerThickness + DetailAltitudeShift_a;
		
		radialdist += ringnoise;
		
		return radialdist;
	}
	
	float GetLayerFalloff(float3 Pos, float layernoise)
	{
		float ringdist = GetNormalizedRingDistance(Pos, layernoise);
		
		return (1.0 - exp(-ringdist * CloudLowerFalloff)) * (1.0 - exp(-(1.0 - ringdist) * CloudUpperFalloff));
	}
	
	float SampleCloudLayer(float3 inPos, float inDetailAmount)
	{
		
		float4 flatsample = CloudMaskTexture.SampleLevel(CloudMaskTextureSampler, (inPos.xy / CloudMaskScale) - OffsetXY, 0);
		
		float detailnoise1 = CloudVolumeTexture.SampleLevel(CloudVolumeTextureSampler, (inPos / DetailScale_a) + Panner, 0 + DetailMip_a).r - 0.5;
		float detailnoise2 = CloudVolumeTexture.SampleLevel(CloudVolumeTextureSampler, (inPos / DetailScale_b) + Panner, 0 + DetailMip_b).r - 0.5;
		
		float volsample = flatsample.r + ((detailnoise1) * inDetailAmount);
		//volsample = max(volsample, flatsample.g + ((detailnoise2) * inDetailAmount * DetailAmount_b));
		
#if UseDetailTexture
			float detailsample = VolumeTex.SampleLevel(VolumeTexSampler, (inPos / Cloud2Divisor) + Pan, 0.0 + Mip2).r;	
			volsample = volsample + ((detailsample - 0.5) * Detail1Amount) - CloudMaskParams.g;
#else
		volsample = volsample - CloudMaskBias;
#endif
		
		float falloff = GetLayerFalloff(inPos, (volsample - 1.0) * DetailAltitudeShift_a);
		volsample -= (1.0 / max(0.01, falloff)) * CloudMaskBias;
		volsample *= CloudDensity;
		volsample *= saturate(falloff);
		//volsample = 1 - exp(-volsample * ExtraFog);

		return volsample;
	}
	
	float4 RayMarch(float3 Pos, float3 Dir, int Steps, float MaxDist)
	{
		float accum = 0;
		float dist = 0;
		float3 lightenergy = 0;
		float cursample, lastsample, curdensity = 0;
	
		for (int i = 0; i < Steps; i++)
		{
            //initialstepsize *= 1.05;
		
			cursample = SampleCloudLayer(Pos, DetailAmount_a);
			
            //dist += StepSize;
			
			if (cursample > 0)
			{
			
				accum += cursample;
			
				float3 lpos = Pos;
				lpos -= LV * ShadowStep;
				float shadowdist = 0;
				float shadowlength = 0;
				
				for (int shadowstep = 0; shadowstep < NumShadowSteps; shadowstep++)
				{
					
					//float2 curnoise = MaterialExpressionVectorNoise(float3(shadowstep, i, 0), 1.00000000, 0.00000000, 0.00000000, 300.00000000).xyz;
					
					lpos += LV * ShadowStep;
					//shadowlength += shadowstepsize;
					float lsample = SampleCloudLayer(lpos, DetailAmount_a);
					
					shadowdist += max(0, lsample);
					
				}
				
				curdensity = 1.0 - exp(-max(0.0, cursample) * DensityStep);
				
				lightenergy += exp(-shadowdist * ShadowDensity * ShadowStep) * curdensity * SunColor * Transmittance;
				
			#if 1
				//float scattersample = CloudVolumeTexture.SampleLevel(CloudVolumeTextureSampler, (Pos / CloudMaskScale) + Panner, 0.0 + DetailMip_a).r;
				//scattersample = 1.0 - saturate(SampleCloudLayer(Pos, DetailAmount_a * 0.5));
				
				//scattersample = exp(-pow((1.0 - scattersample), InternalScatterPower) * InternalScatterBoost);

				float4 scattervalues = ScatterDensityOctaves;
				//texture based scattering variance
					//scattervalues.g *= scattersample;
					//scattervalues.b *= scattersample * scattersample;
					//scattervalues.a *= scattersample * scattersample * scattersample;
				
				float4 scatterenergy = exp(-shadowdist * ShadowDensity * ShadowStep * scattervalues);
				
				
				//scatterenergy *= 1.0 - exp(-(curdensity + (pow((1.0 - scattersample), InternalScatterPower) * InternalScatterBoost)) * PowderDensity);
				
				lightenergy += dot(scatterenergy, ScatterIntensityOctaves) * curdensity * SunColor * Transmittance;
				
			#endif
			#if 0
					lightenergy += exp(-shadowdist * ShadowDensity * shadowstepsize) * curdensity * SunColor * transmittance;
				
					float scattersample = VolumeTex.SampleLevel(VolumeTexSampler, (Pos / CloudDivisor) + Pan, 0.0 + Mip1).r;
					scattersample = exp(-pow((1 - scattersample), ScatteringParams.g) * ScatteringParams.r);
		
					shadowdist *= scattersample;
					lightenergy += exp(-shadowdist * ScatteringParams.a * shadowstepsize) * curdensity * SunColor * transmittance * ScatteringParams.b;
			#endif
				
				
				
				//do transmittance after ambience
				//transmittance *= 1 - curdensity;

				if (Transmittance < 0.001)
				{
					//transmittance = 0;
					//return float4(lightenergy, accum);
				}
				
#if 1 //Sky Lighting
				lpos = Pos + normalize(Pos) * DensityStep;
				float lsample = SampleCloudLayer(lpos, DetailAmount_a);
				shadowdist = lsample;
				lpos = Pos + normalize(Pos) * DensityStep * 2.0;
				lsample = SampleCloudLayer(lpos, DetailAmount_a);
				shadowdist += lsample;
				shadowdist = max(0, shadowdist);
				
				lightenergy += exp(-shadowdist * AmbientDensity * DensityStep) * curdensity * SkyColor * Transmittance;
				
#endif
				
#if 1 // ground bounce
				shadowdist = max(0, cursample);
				lpos = Pos - normalize(Pos) * DensityStep;
				lsample = SampleCloudLayer(lpos, DetailAmount_a);
				shadowdist = max(0, lsample);
				lpos = Pos - normalize(Pos) * DensityStep * 2;
				lsample = SampleCloudLayer(lpos, DetailAmount_a);
				shadowdist += max(0, lsample);
				shadowdist = max(0, shadowdist);
				//shadowdist = max(shadowdist, lsample + cursample);
				
				lightenergy += exp(-shadowdist * AmbientDensity * DensityStep) * curdensity * GroundColor * Transmittance * dot(0.3, SkyColor);
				
#endif
				
				Transmittance *= 1.0 - curdensity;
			}
			
			if (dist > MaxDist)
			{
				//	Pos += Dir * fmod(StepSize, MaxDist);
				//	cursample = PseudoVolumeTexture(Tex, TexSampler, Pos / CloudDivisor, 16, 256).r;
				//	//accum += cursample * fmod(StepSize, MaxDist);
				return float4(lightenergy, accum);
			}
			
			Pos += Dir * DensityStep;
			//DensityStep = max(initialstepsize, ((-cursample) * CloudDivisor * raycastdist) / CloudDensity * (dist / MaxDist));
			
			accumdist += DensityStep * Transmittance;
		}
		
		return float4(lightenergy, Transmittance);
	}
	
	float4 Render()
	{
		float LayerABottom = GroundRadius + CloudLayerAltitude;
		float LayerATop = LayerABottom + CloudLayerThickness;
		
		float3 PlanetOuter = SphereIntersection(GroundRadius);
		
		//Use Planetary Distance beyond depth precision limits
		if (SceneDepth > 650000 && PlanetOuter.x > 0)
		{
			SceneDepth = PlanetOuter.x;
		}
		
		float3 LayerAOuter = SphereIntersection(LayerATop);
		float3 LayerAInner = SphereIntersection(LayerABottom);
		
		float cloudentry = LayerAOuter.x;
		float cloudexit = LayerAOuter.y;
		
		//extract rays that exit and re-enter the cloud layer
		//Is Camera outisde the Inner Cloud layer
		float mask = (LayerAInner.x > 0) ? 1 : 0;
		//ignore rays that hit the planet surface
		mask *= (PlanetOuter.x > 0) ? 0 : 1;
		
		//Ray exits and re-enters, stop the initial ray at that first exit spot, so we can skip empty space inbetween easily
		if (mask == 1)
		{
			cloudexit = min(LayerAInner.x, SceneDepth);
		}
	
		
		// Ray Hits the Inner Shell of Cloud layer
		if (LayerAInner.z > 0)
		{
			cloudexit = min(LayerAInner.x, SceneDepth);
		}
		
		//Camera is inside the bottom/inner cloud radius, advacnce ray to inner shell
		if (LayerAInner.x == 0 && LayerAInner.z > 0)
		{
		
			//LayerAInner.y 
		
		
			cloudentry = LayerAInner.y;
			cloudexit = LayerAOuter.y;
		}
		
		float4 clouds = 0;
		float clouddist = cloudexit - cloudentry;
		float tracedist = 0;
		
		if (clouddist > 0)
		{
			accumdist = cloudentry;
			//StepSize = clouddist / MaxSteps;
			
			
			float calcstep = ceil(clouddist / DensityStep);
			
			
			clouds = RayMarch(RO + cloudentry * RD, RD, Steps, clouddist);
		}
		
		if (mask == 1 && LayerAInner.y < SceneDepth)
		{
			cloudentry = min(LayerAInner.y, SceneDepth);
			cloudexit = min(LayerAOuter.y, SceneDepth);
			
			//skipdist = cloudentry;
			
			tracedist = cloudexit - cloudentry;
			//StepSize = tracedist / Steps;
			clouds += RayMarch(RO + cloudentry * RD, RD, Steps, tracedist);
			
			clouddist += tracedist;
			clouddist += LayerAInner.z;
			
		}
		
		clouddist /= 2.0 * GroundRadius;
	
#if 0
			accumdist /= 10000;
			float aerialp = (1-exp(-accumdist * AtmosphereDensity));
			transmittance = lerp(transmittance, 1.0, aerialp);
			clouds.rgb *= 1.0-aerialp;
			clouds.rgb += (SunColor + SkyColor) * aerialp * ExtraFog;
#else
		float3 appos = RO + (accumdist * RD) - float3(0, 0, GroundRadius);;
		float4 ap = MaterialExpressionSkyAtmosphereAerialPerspective(Parameters, appos);
		clouds.rgb *= ap.a;
		clouds.rgb += ap.rgb * 1.0 * (1.0 - Transmittance);
		
			
#endif
		
		
		
		return float4(clouds.rgb, Transmittance);
	}
	
};

struct CloudSceneMaker
{
	
	CloudScene NewClouds;
	
	CloudScene MakeCloudLayer(
		FMaterialPixelParameters InParameters, 
		float inPlanetRadius, 
		float inSteps, 
		float inStepSize, 
		Texture2D inCloudMask, 
		SamplerState inMaskSampler, 
		float4 inCloudBaseParams, 
		float4 inFalloffParams, 
		Texture3D inVolTex, 
		SamplerState inVolSampler, 
		float4 inDetailParamsA, 
		float4 inDetailParamsB,
		float3 inSunColor,
		float3 inSkyColor,
		float3 inGroundColor,
		float4 inScatteringParams,
		float3 inPanner,
		float4 inDensityParams,
		float3 inRO,
		float3 inRD,
		float3 inLV,
		float inSceneDepth,
		float inTanHalfFOV,
		float2 inViewSize)
	{
		
		NewClouds.Parameters = InParameters;
		NewClouds.GroundRadius = inPlanetRadius;
		NewClouds.Steps = inSteps;
		NewClouds.DensityStep = inStepSize * min(4, 1 / abs(inRD.z));
		NewClouds.ShadowStep = inStepSize * min(8, 1 / abs(inLV.z));
		NewClouds.CloudMaskTexture = inCloudMask;
		NewClouds.CloudMaskTextureSampler = inMaskSampler;
		NewClouds.CloudMaskScale = inCloudBaseParams.r;
		NewClouds.CloudMaskBias = inCloudBaseParams.g;
		NewClouds.OffsetXY = inCloudBaseParams.ba;
		NewClouds.CloudLayerAltitude = inFalloffParams.r;
		NewClouds.CloudLayerThickness = inFalloffParams.g;
		NewClouds.CloudUpperFalloff = inFalloffParams.b;
		NewClouds.CloudLowerFalloff = inFalloffParams.a;
		NewClouds.CloudVolumeTexture = inVolTex;
		NewClouds.CloudVolumeTextureSampler = inVolSampler;
		NewClouds.DetailScale_a = inDetailParamsA.r;
		NewClouds.DetailAmount_a = inDetailParamsA.g;
		NewClouds.DetailMip_a = inDetailParamsA.b;
		NewClouds.DetailAltitudeShift_a = inDetailParamsA.a;
		NewClouds.DetailScale_b = inDetailParamsB.r;
		NewClouds.DetailAmount_b = inDetailParamsB.g;
		NewClouds.DetailMip_b = inDetailParamsB.b;
		NewClouds.SunColor = inSunColor;
		NewClouds.SkyColor = inSkyColor;
		NewClouds.GroundColor = inGroundColor;
		NewClouds.InternalScatterBoost = inScatteringParams.r;
		NewClouds.InternalScatterPower = inScatteringParams.g;
		NewClouds.ScatterIntensityOctaves = float4(1, inScatteringParams.b, inScatteringParams.b * inScatteringParams.b, inScatteringParams.b * inScatteringParams.b * inScatteringParams.b);
		NewClouds.ScatterDensityOctaves = float4(1, inScatteringParams.a, inScatteringParams.a * inScatteringParams.a, inScatteringParams.a * inScatteringParams.a * inScatteringParams.a);
		NewClouds.Panner = inPanner;
		NewClouds.CloudDensity = inDensityParams.r / 1000;
		NewClouds.ShadowDensity = inDensityParams.g;
		NewClouds.AmbientDensity = inDensityParams.b;
		NewClouds.PowderDensity = inDensityParams.a;
		NewClouds.RO = inRO;
		NewClouds.RD = inRD;
		NewClouds.LV = inLV;
		NewClouds.SceneDepth = inSceneDepth;
		NewClouds.TanHalfFOV = inTanHalfFOV;
		NewClouds.VolumeResolution = 256;
		NewClouds.ViewSize = inViewSize;
		NewClouds.LightEnergy = 0;
		NewClouds.Transmittance = 1;
		NewClouds.accumdist = 0;
		
		return NewClouds;
	}
};

CloudSceneMaker CloudMaker;
CloudScene Clouds = CloudMaker.MakeCloudLayer(Parameters, PlanetRadius, MaxSteps, StepSize, CloudMask, CloudMaskSampler, CloudMaskParams, FalloffParams, VolumeTex, VolumeTexSampler, Detail1Params, Detail2Params, SunColor, SkyColor, GroundColor, ScatteringParams, Pan, DensityParams, RayOrigin, RayDirection, LightVector, SD, TanHalfFOV, ViewRes);

return Clouds.Render();