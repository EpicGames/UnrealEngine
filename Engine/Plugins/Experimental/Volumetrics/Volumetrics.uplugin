{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Volumetrics",
	"Description": "A library of volume creation and rendering tools using Blueprints.",
	"Category": "Rendering",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Plugins": [
		{
			"Name": "BlueprintMaterialTextureNodes",
			"Enabled": true
		},
		{
			"Name": "Landmass",
			"Enabled": true
		},
		{
			"Name": "Niagara",
			"Enabled": true
		}
	]
}